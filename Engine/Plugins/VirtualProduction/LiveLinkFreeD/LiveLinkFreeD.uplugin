{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkFreeD",
	"Description": "Live Link plugin for the FreeD protocol",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Plugins": [
		{
			"Name": "UdpMessaging",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "LiveLinkFreeD",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"ProgramAllowList": [ "LiveLinkHub" ]
		}
	]
}
 