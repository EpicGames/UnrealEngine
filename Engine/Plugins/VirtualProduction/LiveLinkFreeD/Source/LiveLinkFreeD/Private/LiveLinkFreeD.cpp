// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFreeD.h"

#define LOCTEXT_NAMESPACE "LiveLinkFreeDModule"

DEFINE_LOG_CATEGORY(LogLiveLinkFreeD);

void FLiveLinkFreeDModule::StartupModule()
{
}

void FLiveLinkFreeDModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLiveLinkFreeDModule, LiveLinkFreeD)