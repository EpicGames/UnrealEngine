// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkFreeDConnectionSettings.generated.h"

USTRUCT()
struct LIVELINKFREED_API FLiveLinkFreeDConnectionSettings
{
	GENERATED_BODY()

public:

	/**
	 * Local interface IP address.
	 * 0.0.0.0 (INADDR_ANY) binds to all local network interfaces.
	 */
	UPROPERTY(EditAnywhere, Category = "Connection Settings", meta = (DisplayName = "Local IP Address"))
	FString IPAddress = TEXT("0.0.0.0");

	/** UDP port number */
	UPROPERTY(EditAnywhere, Category = "Connection Settings", meta = (DisplayName = "UDP Port Number"))
	uint16 UDPPortNumber = 40000;

	/** Custom subject name (if empty, defaults to "Camera <Camera ID>") */
	UPROPERTY(EditAnywhere, Category = "Connection Settings", meta = (DisplayName = "Subject Name"))
	FString SubjectName = TEXT("");
};
