// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

//
// Free-D protocol packet data definitions as per the official spec:
// https://www.manualsdir.com/manuals/641433/vinten-radamec-free-d.html
//
struct FreeDPacketDefinition
{
	static const uint8 PacketTypeD1;
	static const uint8 PacketSizeD1;

	static const uint8 PacketType;
	static const uint8 CameraID;
	static const uint8 Yaw;
	static const uint8 Pitch;
	static const uint8 Roll;
	static const uint8 X;
	static const uint8 Y;
	static const uint8 Z;
	static const uint8 FocalLength;
	static const uint8 FocusDistance;
	static const uint8 UserDefined;
	static const uint8 Checksum;
};
