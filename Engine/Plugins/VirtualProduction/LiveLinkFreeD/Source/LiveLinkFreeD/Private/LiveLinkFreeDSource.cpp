// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFreeDSource.h"
#include "LiveLinkFreeD.h"
#include "ILiveLinkClient.h"
#include "Async/Async.h"
#include "LiveLinkFreeDConnectionSettings.h"
#include "Misc/CoreDelegates.h"
#include "LiveLinkFreeDPacketInfo.h"
#include "Roles/LiveLinkCameraRole.h"

#include "Common/UdpSocketBuilder.h"
#include "Roles/LiveLinkCameraTypes.h"

#define LOCTEXT_NAMESPACE "LiveLinkFreeDSourceFactory"

// These consts must be defined here in the CPP for non-MS compiler issues
const uint8 FreeDPacketDefinition::PacketTypeD1 = 0xD1;
const uint8 FreeDPacketDefinition::PacketSizeD1 = 0x1D;
const uint8 FreeDPacketDefinition::PacketType = 0x00;
const uint8 FreeDPacketDefinition::CameraID = 0x01;
const uint8 FreeDPacketDefinition::Yaw = 0x02;
const uint8 FreeDPacketDefinition::Pitch = 0x05;
const uint8 FreeDPacketDefinition::Roll = 0x08;
const uint8 FreeDPacketDefinition::X = 0x0B;
const uint8 FreeDPacketDefinition::Y = 0x0E;
const uint8 FreeDPacketDefinition::Z = 0x11;
const uint8 FreeDPacketDefinition::FocalLength = 0x14;
const uint8 FreeDPacketDefinition::FocusDistance = 0x17;
const uint8 FreeDPacketDefinition::UserDefined = 0x1A;
const uint8 FreeDPacketDefinition::Checksum = 0x1C;

FLiveLinkFreeDSource::FLiveLinkFreeDSource(const FLiveLinkFreeDConnectionSettings& ConnectionSettings)
: Client(nullptr)
, Stopping(false)
, Thread(nullptr)
, CustomSubjectName(ConnectionSettings.SubjectName)
{
	SourceStatus = LOCTEXT("SourceStatus_NoData", "No data");
	SourceType = LOCTEXT("SourceType_FreeD", "FreeD");
	SourceMachineName = FText::Format(LOCTEXT("FreeDSourceMachineName", "{0}:{1}"), FText::FromString(ConnectionSettings.IPAddress), FText::AsNumber(ConnectionSettings.UDPPortNumber, &FNumberFormattingOptions::DefaultNoGrouping()));

	FIPv4Address::Parse(ConnectionSettings.IPAddress, DeviceEndpoint.Address);
	DeviceEndpoint.Port = ConnectionSettings.UDPPortNumber;

	Socket = FUdpSocketBuilder(TEXT("FreeDListenerSocket"))
		.AsNonBlocking()
		.AsReusable()
		.BoundToEndpoint(DeviceEndpoint)
		.WithReceiveBufferSize(ReceiveBufferSize);

	if ((Socket != nullptr) && (Socket->GetSocketType() == SOCKTYPE_Datagram))
	{
		ReceiveBuffer.SetNumUninitialized(ReceiveBufferSize);
		SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		DeferredStartDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FLiveLinkFreeDSource::Start);

		UE_LOGF(LogLiveLinkFreeD, Log, "LiveLinkFreeDSource: Opened UDP socket with IP address %ls", *DeviceEndpoint.ToString());
	}
	else
	{
		UE_LOGF(LogLiveLinkFreeD, Error, "LiveLinkFreeDSource: Failed to open UDP socket with IP address %ls", *DeviceEndpoint.ToString());
	}
}

FLiveLinkFreeDSource::~FLiveLinkFreeDSource()
{
	// This could happen if the object is destroyed before FCoreDelegates::OnEndFrame calls FLiveLinkFreeDSource::Start
	if (DeferredStartDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(DeferredStartDelegateHandle);
	}

	Stop();

	if (Thread != nullptr)
	{
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}

	if (Socket != nullptr)
	{
		Socket->Close();
		SocketSubsystem->DestroySocket(Socket);
		Socket = nullptr;
	}
}

void FLiveLinkFreeDSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;
}

void FLiveLinkFreeDSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	// Save our source settings pointer so we can use it directly
	SavedSourceSettings = Cast<ULiveLinkFreeDSourceSettings>(Settings);
}

bool FLiveLinkFreeDSource::IsSourceStillValid() const
{
	// Source is valid if we have a valid thread
	bool bIsSourceValid = !Stopping && (Thread != nullptr) && (Socket != nullptr);
	return bIsSourceValid;
}

bool FLiveLinkFreeDSource::RequestSourceShutdown()
{
	Stop();

	return true;
}

//
// Specific manufacturer default data (we still use auto-ranging as the default)
//
// Name			UDP port	Zoom (wide)	(tele)		Focus (near)	(far)		Spare
// Generic		40000		0x0			0x10000		0x0				0x10000		Unused
// Panasonic	1111		0x555		0xfff		0x555			0xfff		Iris 0x555 (close) - 0xfff (open)
// Sony			40000		0x0			0xFFFFFF	0x7FFFFF		0x000000	Lower 12 bits - Iris (F value * 100); Upper 4 bits - Frame number
// Mosys		8001		0x0			0xffff		0x0				0xffff		Lower 8 bits - Tracking quality 0-3 (undef, good, caution, bad)
// Stype		6301		0x0			0xffffff	0x0				0xffffff	Unused
// Ncam			6301		0x0			0xffffff	0x0				0xffffff	Unused
//

void FLiveLinkFreeDSource::OnSettingsChanged(ULiveLinkSourceSettings* Settings, const FPropertyChangedEvent& PropertyChangedEvent)
{
	ILiveLinkSource::OnSettingsChanged(Settings, PropertyChangedEvent);

	FProperty* MemberProperty = PropertyChangedEvent.MemberProperty;
	FProperty* Property = PropertyChangedEvent.Property;
	if (Property && MemberProperty && (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive))
	{
		ULiveLinkFreeDSourceSettings* SourceSettings = Cast<ULiveLinkFreeDSourceSettings>(Settings);
		if (SavedSourceSettings != SourceSettings)
		{
			UE_LOGF(LogLiveLinkFreeD, Error, "LiveLinkFreeDSource: OnSettingsChanged pointers don't match - this should never happen!");
			return;
		}

		if (SourceSettings != nullptr)
		{
			static FName NAME_DefaultConfig = GET_MEMBER_NAME_CHECKED(ULiveLinkFreeDSourceSettings, DefaultConfig);
			static FName NAME_FocusDistanceEncoderData = GET_MEMBER_NAME_CHECKED(ULiveLinkFreeDSourceSettings, FocusDistanceEncoderData);
			static FName NAME_FocalLengthEncoderData = GET_MEMBER_NAME_CHECKED(ULiveLinkFreeDSourceSettings, FocalLengthEncoderData);
			static FName NAME_UserDefinedEncoderData = GET_MEMBER_NAME_CHECKED(ULiveLinkFreeDSourceSettings, UserDefinedEncoderData);
			const FName PropertyName = Property->GetFName();
			const FName MemberPropertyName = MemberProperty->GetFName();

			bool bFocusDistanceEncoderDataChanged = false;
			bool bFocalLengthEncoderDataChanged = false;
			bool bUserDefinedEncoderDataChanged = false;

			if (PropertyName == NAME_DefaultConfig)
			{
				switch (SourceSettings->DefaultConfig)
				{
					case EFreeDDefaultConfigs::Generic:		SourceSettings->FocusDistanceEncoderData = FFreeDEncoderData({ true, false, false, 0, 0xffff, 0x0000ffff });
															SourceSettings->FocalLengthEncoderData = FFreeDEncoderData({ true, false, false, 0, 0xffff, 0x0000ffff });
															SourceSettings->UserDefinedEncoderData = FFreeDEncoderData({ false });
															break;

					case EFreeDDefaultConfigs::Panasonic:	SourceSettings->FocusDistanceEncoderData = FFreeDEncoderData({ true, false, false, 0x0555, 0x0fff, 0x0000ffff });
															SourceSettings->FocalLengthEncoderData = FFreeDEncoderData({ true, false, false, 0x0555, 0x0fff, 0x0000ffff });
															SourceSettings->UserDefinedEncoderData = FFreeDEncoderData({ true, true, false, 0x0555, 0x0fff, 0x0000ffff });
															break;

					case EFreeDDefaultConfigs::Sony:		SourceSettings->FocusDistanceEncoderData = FFreeDEncoderData({ true, true, false, 0, 0x007fffff, 0x00ffffff });
															SourceSettings->FocalLengthEncoderData = FFreeDEncoderData({ true, false, false, 0, 0x00ffffff, 0x00ffffff });
															SourceSettings->UserDefinedEncoderData = FFreeDEncoderData({ true, false, false, 0, 0x0fff, 0x00000fff });
															break;

					case EFreeDDefaultConfigs::Mosys:		SourceSettings->FocusDistanceEncoderData = FFreeDEncoderData({ true, false, false, 0, 0xffff, 0x0000ffff });
															SourceSettings->FocalLengthEncoderData = FFreeDEncoderData({ true, false, false, 0, 0xffff, 0x0000ffff });
															SourceSettings->UserDefinedEncoderData = FFreeDEncoderData({ false });
															break;

					case EFreeDDefaultConfigs::Stype:		SourceSettings->FocusDistanceEncoderData = FFreeDEncoderData({ true, false, false, 0, 0xffffff, 0x00ffffff });
															SourceSettings->FocalLengthEncoderData = FFreeDEncoderData({ true, false, false, 0, 0xffffff, 0x00ffffff });
															SourceSettings->UserDefinedEncoderData = FFreeDEncoderData({ false });
															break;

					case EFreeDDefaultConfigs::Ncam:		SourceSettings->FocusDistanceEncoderData = FFreeDEncoderData({ true, false, false, 0, 0xffffff, 0x00ffffff });
															SourceSettings->FocalLengthEncoderData = FFreeDEncoderData({ true, false, false, 0, 0xffffff, 0x00ffffff });
															SourceSettings->UserDefinedEncoderData = FFreeDEncoderData({ false });
															break;
				}

				bFocusDistanceEncoderDataChanged = true;
				bFocalLengthEncoderDataChanged = true;
				bUserDefinedEncoderDataChanged = true;
			}

			if (MemberPropertyName == NAME_FocusDistanceEncoderData)
			{
				bFocusDistanceEncoderDataChanged = true;
			}
			else if (MemberPropertyName == NAME_FocalLengthEncoderData)
			{
				bFocalLengthEncoderDataChanged = true;
			}
			else if (MemberPropertyName == NAME_UserDefinedEncoderData)
			{
				bUserDefinedEncoderDataChanged = true;
			}

			if (bFocusDistanceEncoderDataChanged)
			{
				UpdateEncoderData(&SourceSettings->FocusDistanceEncoderData);
			}

			if (bFocalLengthEncoderDataChanged)
			{
				UpdateEncoderData(&SourceSettings->FocalLengthEncoderData);
			}

			if (bUserDefinedEncoderDataChanged)
			{
				UpdateEncoderData(&SourceSettings->UserDefinedEncoderData);
			}
		}
	}
}

void FLiveLinkFreeDSource::UpdateEncoderData(FFreeDEncoderData* InEncoderData)
{
	if (InEncoderData->Min == InEncoderData->Max)
	{
		UE_LOGF(LogLiveLinkFreeD, Error, "LiveLinkFreeDSource: EncoderData Min/Max values can't be equal (you may need to rack your encoder) - incoming data may be invalid!");
	}

	// Update any changed encoder data to reset min/max values for auto-ranging
	if (!InEncoderData->bUseManualRange)
	{
		InEncoderData->Min = InEncoderData->MaskBits;
		InEncoderData->Max = 0;
	}

	// Remove the static data from the EncounteredSubjects list so it will be automatically updated during the next Send()
	EncounteredSubjects.Remove(FName(CameraSubjectName));
}

// FRunnable interface
void FLiveLinkFreeDSource::Start()
{
	check(DeferredStartDelegateHandle.IsValid());

	FCoreDelegates::OnEndFrame.Remove(DeferredStartDelegateHandle);
	DeferredStartDelegateHandle.Reset();

	SourceStatus = LOCTEXT("SourceStatus_Receiving", "Receiving");

	ThreadName = "LiveLinkFreeD Receiver ";
	ThreadName.AppendInt(FAsyncThreadIndex::GetNext());

	Thread = FRunnableThread::Create(this, *ThreadName, 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());
}

void FLiveLinkFreeDSource::Stop()
{
	Stopping = true;
}

uint32 FLiveLinkFreeDSource::Run()
{
	// Free-D max data rate is 100Hz
	const FTimespan SocketTimeout(FTimespan::FromMilliseconds(10));

	while (!Stopping)
	{
		if (Socket && Socket->Wait(ESocketWaitConditions::WaitForRead, SocketTimeout))
		{
			uint32 PendingDataSize = 0;
			while (Socket && Socket->HasPendingData(PendingDataSize))
			{
				int32 ReceivedDataSize = 0;
				if (Socket && Socket->Recv(ReceiveBuffer.GetData(), ReceiveBufferSize, ReceivedDataSize))
				{
					if (ReceivedDataSize > 0)
					{
						if (SavedSourceSettings == nullptr)
						{
							UE_LOGF(LogLiveLinkFreeD, Error, "LiveLinkFreeDSource: Received a packet, but we don't have a valid SavedSourceSettings!");
						}
						else if (ReceiveBuffer[FreeDPacketDefinition::PacketType] == FreeDPacketDefinition::PacketTypeD1)
						{
							// The only message that we care about is the 0xD1 message which contains PnO data, zoom, focus, and a user defined field (usually iris)
							uint8 CameraId = ReceiveBuffer[FreeDPacketDefinition::CameraID];
							FRotator Orientation;
							Orientation.Yaw = Decode_Signed_8_15(&ReceiveBuffer[FreeDPacketDefinition::Yaw]);
							Orientation.Pitch = Decode_Signed_8_15(&ReceiveBuffer[FreeDPacketDefinition::Pitch]);
							Orientation.Roll = Decode_Signed_8_15(&ReceiveBuffer[FreeDPacketDefinition::Roll]);

							// FreeD has the X and Y axes flipped from Unreal
							FVector Position;
							Position.X = Decode_Signed_17_6(&ReceiveBuffer[FreeDPacketDefinition::Y]);
							Position.Y = Decode_Signed_17_6(&ReceiveBuffer[FreeDPacketDefinition::X]);
							Position.Z = Decode_Signed_17_6(&ReceiveBuffer[FreeDPacketDefinition::Z]);

							int32 FocalLengthInt = Decode_Unsigned_24(&ReceiveBuffer[FreeDPacketDefinition::FocalLength]);
							int32 FocusDistanceInt = Decode_Unsigned_24(&ReceiveBuffer[FreeDPacketDefinition::FocusDistance]);
							int32 UserDefinedDataInt = Decode_Unsigned_16(&ReceiveBuffer[FreeDPacketDefinition::UserDefined]);

							float FocalLength = ProcessEncoderData(SavedSourceSettings->FocalLengthEncoderData, FocalLengthInt);
							float FocusDistance = ProcessEncoderData(SavedSourceSettings->FocusDistanceEncoderData, FocusDistanceInt);
							float UserDefinedData = ProcessEncoderData(SavedSourceSettings->UserDefinedEncoderData, UserDefinedDataInt);

							uint8 Checksum = CalculateChecksum(&ReceiveBuffer[FreeDPacketDefinition::PacketType], ReceivedDataSize - 1);
							if (Checksum != ReceiveBuffer[FreeDPacketDefinition::Checksum])
							{
								UE_LOGF(LogLiveLinkFreeD, Warning, "LiveLinkFreeDSource: Received packet checksum error - received 0x%02x, calculated 0x%02x", ReceiveBuffer[FreeDPacketDefinition::Checksum], Checksum);
							}

							if (ReceivedDataSize != FreeDPacketDefinition::PacketSizeD1)
							{
								UE_LOGF(LogLiveLinkFreeD, Warning, "LiveLinkFreeDSource: Received packet length mismatch - received 0x%02x, calculated 0x%02x", ReceivedDataSize, FreeDPacketDefinition::PacketSizeD1);
							}

							FLiveLinkFrameDataStruct FrameData(FLiveLinkCameraFrameData::StaticStruct());
							FLiveLinkCameraFrameData* CameraFrameData = FrameData.Cast<FLiveLinkCameraFrameData>();
							CameraFrameData->Transform = FTransform(Orientation, Position);

							if (SavedSourceSettings->bSendExtraMetaData)
							{
								CameraFrameData->MetaData.StringMetaData.Add(FName(TEXT("CameraId")), FString::Printf(TEXT("%d"), CameraId));
								CameraFrameData->MetaData.StringMetaData.Add(FName(TEXT("FrameCounter")), FString::Printf(TEXT("%d"), FrameCounter));
							}

							if (SavedSourceSettings->FocalLengthEncoderData.bIsValid)
							{
								CameraFrameData->FocalLength = FocalLength;
							}
							if (SavedSourceSettings->FocusDistanceEncoderData.bIsValid)
							{
								CameraFrameData->FocusDistance = FocusDistance;
							}
							if (SavedSourceSettings->UserDefinedEncoderData.bIsValid)
							{
								CameraFrameData->Aperture = UserDefinedData;
							}

							if (CustomSubjectName.IsEmpty())
							{
								CameraSubjectName = FString::Printf(TEXT("Camera %d"), CameraId);
							}
							else
							{
								CameraSubjectName = CustomSubjectName;
							}

							Send(&FrameData, FName(CameraSubjectName));

							FrameCounter++;
						}
						else
						{
							UE_LOGF(LogLiveLinkFreeD, Warning, "LiveLinkFreeDSource: Unsupported FreeD message type 0x%02x", ReceiveBuffer[FreeDPacketDefinition::PacketType]);
						}
					}
				}
			}
		}
	}
	
	return 0;
}

void FLiveLinkFreeDSource::Send(FLiveLinkFrameDataStruct* FrameDataToSend, FName SubjectName)
{
	if (Stopping || (Client == nullptr))
	{
		return;
	}

	if (!EncounteredSubjects.Contains(SubjectName))
	{
		FLiveLinkStaticDataStruct StaticData(FLiveLinkCameraStaticData::StaticStruct());

		FLiveLinkCameraStaticData& CameraData = *StaticData.Cast<FLiveLinkCameraStaticData>();
		CameraData.bIsFocusDistanceSupported = SavedSourceSettings->FocusDistanceEncoderData.bIsValid;
		CameraData.bIsFocalLengthSupported = SavedSourceSettings->FocalLengthEncoderData.bIsValid;
		CameraData.bIsApertureSupported = SavedSourceSettings->UserDefinedEncoderData.bIsValid;
		Client->PushSubjectStaticData_AnyThread({ SourceGuid, SubjectName }, ULiveLinkCameraRole::StaticClass(), MoveTemp(StaticData));
		EncounteredSubjects.Add(SubjectName);
	}

	Client->PushSubjectFrameData_AnyThread({ SourceGuid, SubjectName }, MoveTemp(*FrameDataToSend));
}

float FLiveLinkFreeDSource::ProcessEncoderData(FFreeDEncoderData& EncoderData, int32 RawEncoderValueInt)
{
	float FinalEncoderValue = 0.0f;

	if (EncoderData.bIsValid)
	{
		RawEncoderValueInt &= EncoderData.MaskBits;

		// Auto-range the input
		if (!EncoderData.bUseManualRange)
		{
			if (RawEncoderValueInt < EncoderData.Min)
			{
				EncoderData.Min = RawEncoderValueInt;
			}
			if (RawEncoderValueInt > EncoderData.Max)
			{
				EncoderData.Max = RawEncoderValueInt;
			}
		}

		int32 Delta = EncoderData.Max - EncoderData.Min;
		if (Delta != 0)
		{
			FinalEncoderValue = FMath::Clamp((float)(RawEncoderValueInt - EncoderData.Min) / (float)Delta, 0.0f, 1.0f);
			if (EncoderData.bInvertEncoder)
			{
				FinalEncoderValue = 1.0f - FinalEncoderValue;
			}
		}
	}

	return FinalEncoderValue;
}

float FLiveLinkFreeDSource::Decode_Signed_8_15(uint8* InBytes)
{
	int32 ret = (*InBytes << 16) | (*(InBytes + 1) << 8) | *(InBytes + 2);
	if (*InBytes & 0x80)
	{
		ret -= 0x00ffffff;
	}
	return (float)ret / 32768.0f;
}

float FLiveLinkFreeDSource::Decode_Signed_17_6(uint8* InBytes)
{
	int32 ret = (*InBytes << 16) | (*(InBytes + 1) << 8) | *(InBytes + 2);
	if (*InBytes & 0x80)
	{
		ret -= 0x00ffffff;
	}
	return (float)ret / 640.0f;
}

uint32 FLiveLinkFreeDSource::Decode_Unsigned_24(uint8* InBytes)
{
	uint32 ret = (*InBytes << 16) | (*(InBytes + 1) << 8) | *(InBytes + 2);
	return ret;
}

uint16 FLiveLinkFreeDSource::Decode_Unsigned_16(uint8* InBytes)
{
	uint16 ret = (*InBytes << 8) | *(InBytes + 1);
	return ret;
}

uint8 FLiveLinkFreeDSource::CalculateChecksum(uint8* InBytes, uint32 Size)
{
	uint8 sum = 0x40;
	for (uint32 i = 0; i < Size; i++)
	{
		sum -= *(InBytes + i);
	}
	return sum;
}

#undef LOCTEXT_NAMESPACE
