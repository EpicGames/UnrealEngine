// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFreeDSourceFactory.h"
#include "LiveLinkFreeDSource.h"
#include "SLiveLinkFreeDSourceFactory.h"


#define LOCTEXT_NAMESPACE "LiveLinkFreeDSourceFactory"

FText ULiveLinkFreeDSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "LiveLinkFreeD Source");	
}

FText ULiveLinkFreeDSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Allows creation of multiple LiveLink sources using the FreeD tracking system");
}

TSharedPtr<SWidget> ULiveLinkFreeDSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	return SNew(SLiveLinkFreeDSourceFactory)
		.OnConnectionSettingsAccepted(FOnLiveLinkFreeDConnectionSettingsAccepted::CreateUObject(this, &ULiveLinkFreeDSourceFactory::CreateSourceFromSettings, InOnLiveLinkSourceCreated));
}

TSharedPtr<ILiveLinkSource> ULiveLinkFreeDSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FLiveLinkFreeDConnectionSettings ConnectionSettings;
	if (!ConnectionString.IsEmpty())
	{
		FLiveLinkFreeDConnectionSettings::StaticStruct()->ImportText(*ConnectionString, &ConnectionSettings, nullptr, PPF_None, GLog, TEXT("ULiveLinkFreeDSourceFactory"));
	}
	return MakeShared<FLiveLinkFreeDSource>(ConnectionSettings);
}

void ULiveLinkFreeDSourceFactory::CreateSourceFromSettings(FLiveLinkFreeDConnectionSettings InConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const
{
	FString ConnectionString;
	FLiveLinkFreeDConnectionSettings::StaticStruct()->ExportText(ConnectionString, &InConnectionSettings, nullptr, nullptr, PPF_None, nullptr);

	TSharedPtr<FLiveLinkFreeDSource> SharedPtr = MakeShared<FLiveLinkFreeDSource>(InConnectionSettings);
	OnSourceCreated.ExecuteIfBound(SharedPtr, MoveTemp(ConnectionString));
}

#undef LOCTEXT_NAMESPACE
