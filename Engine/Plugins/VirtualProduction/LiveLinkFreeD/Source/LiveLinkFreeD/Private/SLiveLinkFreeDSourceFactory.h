// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkFreeDConnectionSettings.h"
#include "Widgets/SCompoundWidget.h"

class FStructOnScope;
class IStructureDetailsView;

#if WITH_EDITOR
#endif //WITH_EDITOR


struct FLiveLinkFreeDConnectionSettings;

DECLARE_DELEGATE_OneParam(FOnLiveLinkFreeDConnectionSettingsAccepted, FLiveLinkFreeDConnectionSettings);

class SLiveLinkFreeDSourceFactory : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkFreeDSourceFactory)
	{}
		SLATE_EVENT(FOnLiveLinkFreeDConnectionSettingsAccepted, OnConnectionSettingsAccepted)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);


private:
	FLiveLinkFreeDConnectionSettings ConnectionSettings;

#if WITH_EDITOR
	TSharedPtr<FStructOnScope> StructOnScope;
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
#endif //WITH_EDITOR

	FReply OnSettingsAccepted();
	FOnLiveLinkFreeDConnectionSettingsAccepted OnConnectionSettingsAccepted;
};
