// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkFreeD, Log, All);

class FLiveLinkFreeDModule : public IModuleInterface
{
public:

	static FLiveLinkFreeDModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<FLiveLinkFreeDModule>(TEXT("LiveLinkFreeD"));
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
