// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"
#include "LiveLinkFreeDConnectionSettings.h"
#include "LiveLinkFreeDSourceSettings.h"
#include "Roles/LiveLinkCameraTypes.h"

#include "Delegates/IDelegateInstance.h"
#include "MessageEndpoint.h"
#include "IMessageContext.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/Runnable.h"

#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "LiveLinkFreeDPacketInfo.h"

struct ULiveLinkFreeDSettings;

class ILiveLinkClient;

class LIVELINKFREED_API FLiveLinkFreeDSource : public ILiveLinkSource, public FRunnable, public TSharedFromThis<FLiveLinkFreeDSource>
{
public:

	FLiveLinkFreeDSource(const FLiveLinkFreeDConnectionSettings& ConnectionSettings);

	virtual ~FLiveLinkFreeDSource();

	// Begin ILiveLinkSource Interface
	
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;

	virtual bool IsSourceStillValid() const override;

	virtual bool RequestSourceShutdown() override;

	virtual FText GetSourceType() const override { return SourceType; };
	virtual FText GetSourceMachineName() const override { return SourceMachineName; }
	virtual FText GetSourceStatus() const override { return SourceStatus; }

	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const override { return ULiveLinkFreeDSourceSettings::StaticClass(); }
	virtual void OnSettingsChanged(ULiveLinkSourceSettings* Settings, const FPropertyChangedEvent& PropertyChangedEvent) override;

	// End ILiveLinkSource Interface

	// Begin FRunnable Interface

	virtual bool Init() override { return true; }
	virtual uint32 Run() override;
	void Start();
	virtual void Stop() override;
	virtual void Exit() override { }

	// End FRunnable Interface

	void UpdateEncoderData(FFreeDEncoderData* InEncoderData);
	void Send(FLiveLinkFrameDataStruct* FrameDataToSend, FName SubjectName);
	float ProcessEncoderData(FFreeDEncoderData& EncoderData, int32 RawEncoderValueInt);

	float Decode_Signed_8_15(uint8* InBytes);
	float Decode_Signed_17_6(uint8* InBytes);
	uint32 Decode_Unsigned_24(uint8* InBytes);
	uint16 Decode_Unsigned_16(uint8* InBytes);
	uint8 CalculateChecksum(uint8* InBytes, uint32 Size);

private:
	ILiveLinkClient* Client;

	// Our identifier in LiveLink
	FGuid SourceGuid;

	FMessageAddress ConnectionAddress;

	FText SourceType;
	FText SourceMachineName;
	FText SourceStatus;
	
	// Threadsafe Bool for terminating the main thread loop
	FThreadSafeBool Stopping;
	
	// Thread to run socket operations on
	FRunnableThread* Thread;
	
	// Name of the sockets thread
	FString ThreadName;

	FSocket* Socket;
	ISocketSubsystem* SocketSubsystem;
	FIPv4Endpoint DeviceEndpoint;

	// Size of receive buffer
	const uint32 ReceiveBufferSize = 1024 * 16;

	// Receive buffer for UDP socket
	TArray<uint8> ReceiveBuffer;
	
	// List of subjects we've already encountered
	TSet<FName> EncounteredSubjects;

	// Deferred start delegate handle.
	FDelegateHandle DeferredStartDelegateHandle;

	// Frame counter for incoming data packets
	int32 FrameCounter = 0;

	// LiveLink subject name for this camera data
	FString CameraSubjectName = TEXT("Camera");

	// Custom subject name from connection settings
	FString CustomSubjectName;

	// Pointer to the settings for this source so we don't have to duplicate data
	ULiveLinkFreeDSourceSettings* SavedSourceSettings = nullptr;
};
