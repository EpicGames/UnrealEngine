// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceFactory.h"
#include "LiveLinkFreeDSource.h"
#include "LiveLinkFreeDSourceFactory.generated.h"

UCLASS()
class ULiveLinkFreeDSourceFactory : public ULiveLinkSourceFactory
{
public:
	GENERATED_BODY()

	virtual FText GetSourceDisplayName() const override;
	virtual FText GetSourceTooltip() const override;

	virtual EMenuType GetMenuType() const override { return EMenuType::SubPanel; }
	virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const override;
	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const override;

private:
	void CreateSourceFromSettings(FLiveLinkFreeDConnectionSettings ConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const;
};