// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkSourceSettings.h"
#include "LiveLinkFreeDSourceSettings.generated.h"

USTRUCT()
struct FFreeDEncoderData
{
	GENERATED_BODY()
	
	/** Is this encoder data valid? */
	UPROPERTY(EditAnywhere, Category = "Encoder Data")
	bool bIsValid = false;

	/** Invert the encoder input direction */
	UPROPERTY(EditAnywhere, Category = "Encoder Data", meta = (EditCondition = "bIsValid"))
	bool bInvertEncoder = false;

	/** Use manual Min/Max values for the encoder normalization (normally uses dynamic auto ranging based on inputs) */
	UPROPERTY(EditAnywhere, Category = "Encoder Data", meta = (EditCondition = "bIsValid"))
	bool bUseManualRange = false;

	/** Minimum raw encoder value */
	UPROPERTY(EditAnywhere, Category = "Encoder Data", meta = (ClampMin = 0, ClampMax = 0x00ffffff, EditCondition = "bIsValid && bUseManualRange"))
	int32 Min = 0x00FFFFFF;

	/** Maximum raw encoder value */
	UPROPERTY(EditAnywhere, Category = "Encoder Data", meta = (ClampMin = 0, ClampMax = 0x00ffffff, EditCondition = "bIsValid && bUseManualRange"))
	int32 Max = 0;

	/** Mask bits for raw encoder value */
	UPROPERTY(EditAnywhere, Category = "Encoder Data", meta = (ClampMin = 0, ClampMax = 0x00ffffff, EditCondition = "bIsValid"))
	int32 MaskBits = 0x00FFFFFF;
};

UENUM(BlueprintType)
enum class EFreeDDefaultConfigs : uint8
{
	Generic,
	Panasonic,
	Sony,
	Stype UMETA(DisplayName="stYpe"),
	Mosys,
	Ncam
};

UCLASS()
class LIVELINKFREED_API ULiveLinkFreeDSourceSettings : public ULiveLinkSourceSettings
{
	GENERATED_BODY()

public:
	/** Send extra string meta data (Camera ID and FrameCounter) */
	UPROPERTY(EditAnywhere, Category = "Source")
	bool bSendExtraMetaData = false;

	/** Default configurations for specific manufacturers */
	UPROPERTY(EditAnywhere, Category = "Source")
	EFreeDDefaultConfigs DefaultConfig = EFreeDDefaultConfigs::Generic;

	/** Raw focus distance (in cm) encoder parameters for this camera - 24 bits max */
	UPROPERTY(EditAnywhere, Category = "Source")
	FFreeDEncoderData FocusDistanceEncoderData = { true, false, false, 0x00ffffff, 0, 0x00ffffff };

	/** Raw focal length/zoom (in mm) encoder parameters for this camera - 24 bits max */
	UPROPERTY(EditAnywhere, Category = "Source")
	FFreeDEncoderData FocalLengthEncoderData = { true, false, false, 0x00ffffff, 0, 0x00ffffff };

	/** Raw user defined/spare data encoder (normally used for Aperture) parameters for this camera - 16 bits max */
	UPROPERTY(EditAnywhere, Category = "Source")
	FFreeDEncoderData UserDefinedEncoderData = { false, false, false, 0x0000ffff, 0, 0x0000ffff };
};
